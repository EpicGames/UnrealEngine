{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Blueprint Snap Nodes Prototype",
	"Description": "Prototype of an alternative way to lay out Blueprint nodes in a more compact manner",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "BlueprintSnapNodes",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	],
	"IsExperimentalVersion": true
}