// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class BlueprintSnapNodes : ModuleRules
{
	public BlueprintSnapNodes(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"InputCore",
				"EditorStyle",
				"GraphEditor",
				"BlueprintGraph",
				"EditorFramework",
				"UnrealEd"
				// ... add private dependencies that you statically link with here ...	
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
