// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintSnapNodes.h"
#include "EdGraphUtilities.h"
#include "K2Node_SnapContainer.h"
#include "Modules/ModuleManager.h"
#include "SGraphNodeSnapContainer.h"

#define LOCTEXT_NAMESPACE "FBlueprintSnapNodesModule"

class FBlueprintSnapNodeWidgetFactory : public FGraphPanelNodeFactory
{
	virtual TSharedPtr<SGraphNode> CreateNode(class UEdGraphNode* InNode) const override
	{
		if (UK2Node_SnapContainer* SnapContainer = Cast<UK2Node_SnapContainer>(InNode))
		{
			return SNew(SGraphNodeSnapContainer, SnapContainer);
		}
		return nullptr;
	}
};





void FBlueprintSnapNodesModule::StartupModule()
{
// 	TSharedPtr<FGameplayTagsGraphPanelPinFactory> GameplayTagsGraphPanelPinFactory = MakeShareable(new FGameplayTagsGraphPanelPinFactory());
// 	FEdGraphUtilities::RegisterVisualPinFactory(GameplayTagsGraphPanelPinFactory);

	NodeWidgetFactory = MakeShareable(new FBlueprintSnapNodeWidgetFactory());
	FEdGraphUtilities::RegisterVisualNodeFactory(NodeWidgetFactory);
}

void FBlueprintSnapNodesModule::ShutdownModule()
{
	if (UObjectInitialized())
	{
		FEdGraphUtilities::UnregisterVisualNodeFactory(NodeWidgetFactory);
	}
	NodeWidgetFactory.Reset();
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FBlueprintSnapNodesModule, BlueprintSnapNodes)
