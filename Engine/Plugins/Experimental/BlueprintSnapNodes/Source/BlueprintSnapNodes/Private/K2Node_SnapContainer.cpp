// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_SnapContainer.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "DiffResults.h"
#include "BlueprintNodeSpawner.h"
#include "K2Node_ExecutionSequence.h"
#include "Logging/TokenizedMessage.h"

#define LOCTEXT_NAMESPACE "BlueprintSnapNodes"

template <typename NodeClass>
NodeClass* SpawnNewNode(UEdGraph* ParentGraph, const FVector2D Location)
{
	NodeClass* NewNode = NewObject<NodeClass>(ParentGraph, NodeClass::StaticClass());
	check(NewNode != nullptr);
	NewNode->CreateNewGuid();

	// position the node before invoking PostSpawnDelegate (in case it wishes to modify this positioning)
	NewNode->NodePosX = Location.X;
	NewNode->NodePosY = Location.Y;

	NewNode->SetFlags(RF_Transactional);
	NewNode->AllocateDefaultPins();
	NewNode->PostPlacedNewNode();

	ParentGraph->Modify();

	ParentGraph->AddNode(NewNode, /*bFromUI =*/true, /*bSelectNewNode =*/false);

	return NewNode;
}


/*******************************************************************************
 * Static Helpers
*******************************************************************************/

/**
 * Helper function for deleting all the nodes from a specified graph. Does not
 * delete any tunnel in/out nodes (to preserve the tunnel).
 * 
 * @param  Graph	The graph you want to delete nodes in.
 */
static void DeleteGeneratedNodesInGraph(UEdGraph* Graph)
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(Graph);
	for (int32 NodeIndex = 0; NodeIndex < Graph->Nodes.Num(); )
	{
		UEdGraphNode* Node = Graph->Nodes[NodeIndex];
		if (ExactCast<UK2Node_Tunnel>(Node) != NULL)
		{
			++NodeIndex;
		}
		else
		{
			FBlueprintEditorUtils::RemoveNode(Blueprint, Node, true);
		}
	}
}
/**
 * Sets or clears the error on a specific node. If the ErrorText is empty, then
 * it resets the error state on the node. If actual error text is supplied, 
 * then the node is flagged as having an error, and the string is appended to 
 * the node's error message.
 * 
 * @param  Node			The node you wish to modify.
 * @param  ErrorText	The text you want to append to the node's error message (empty if you want to clear any errors).
 */
static void SetNodeError(UEdGraphNode* Node, FText const& ErrorText)
{
	if (ErrorText.IsEmpty())
	{
		Node->ErrorMsg.Empty();
		Node->ErrorType = EMessageSeverity::Info;
		Node->bHasCompilerMessage = false;
	}
	else if (Node->bHasCompilerMessage)
	{
		Node->ErrorMsg += TEXT("\n") + ErrorText.ToString();
		Node->ErrorType = EMessageSeverity::Error;
	}
	else
	{
		Node->ErrorMsg = ErrorText.ToString();
		Node->ErrorType = EMessageSeverity::Error;
		Node->bHasCompilerMessage = true;
	}
}


/*******************************************************************************
 * FLayoutVisitor
*******************************************************************************/

/**
 * This class is utilized to help layout math expression nodes by traversing the
 * expression tree and cataloging each expression node's depth. From the tree's 
 * depth we can determine the width of the the graph (an where to place each K2 node):
 *
 *    _
 *   |            [_]---[_]
 *   |           /
 * height   [_]--       [_]--[_]---[_]
 *   |           \     /
 *   |_           [_]---[_]
 *
 *		    ^-------depth/width-------^
 */
#if 0
class FLayoutVisitor : public FExpressionVisitor
{
public:
	/** Tracks the horizontal (depth) placement for each expression node encountered */
	TMap<IFExpressionNode*, int32> DepthChart;
	/** Tracks the vertical (height) placement for each expression node encountered */
	TMap<IFExpressionNode*, int32> HeightChart;
	/** Tracks the total height (value) at each depth (key) */
	TMap<int32, int32> DepthHeightLookup;

	/** */
	FLayoutVisitor()
		: CurrentDepth(0)
		, MaximumDepth(0)
	{
	}

	/**
	 * Retrieves the total depth (or graph width) of the previously traversed
	 * expression tree.
	 */
	int32 GetMaximumDepth() const
	{
		return MaximumDepth;
	}

	/**
	 * Resets this tree visitor so that it can accurately parse another 
	 * expression tree (else the results would stack up).
	 */
	void Clear() 
	{
		CurrentDepth = 0;
		MaximumDepth = 0;
		DepthChart.Empty();
		HeightChart.Empty();
		DepthHeightLookup.Empty();
	}

private:
	/**
	 * From the FExpressionVisitor interface, a generic choke point for visiting 
	 * all expression nodes.
	 *
	 * @return True to continue traversing the tree, false to abort. 
	 */
	virtual bool VisitUnhandled(class IFExpressionNode& Node, EVisitPhase Phase) override
	{
		if (Phase == FExpressionVisitor::VISIT_Pre)
		{
			++CurrentDepth;
			MaximumDepth = FMath::Max(CurrentDepth, MaximumDepth);
		}
		else
		{
			if (Phase == FExpressionVisitor::VISIT_Post)
			{
				--CurrentDepth;
			}
			// else leaf

			// CurrentHeight represents how many nodes have already been placed 
			// at this depth
			int32& CurrentHeight = DepthHeightLookup.FindOrAdd(CurrentDepth);

			DepthChart.Add(&Node, CurrentDepth);
			HeightChart.Add(&Node, CurrentHeight);

			// since we just placed another node at this depth, increase the 
			// height count
			++CurrentHeight;
		}

		// let the tree traversal continue! don't abort it!
		return true;
	}

private:
	int32 CurrentDepth;
	int32 MaximumDepth;
};
#endif


/*******************************************************************************
 * UK2Node_SnapContainer
*******************************************************************************/

UK2Node_SnapContainer::UK2Node_SnapContainer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_SnapContainer::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UK2Node_SnapContainer::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

TSharedPtr<class INameValidatorInterface> UK2Node_SnapContainer::MakeNameValidator() const
{
	// we'll let our parser mark the node for errors after the face (once the 
	// name is submitted)... parsing it with every character could be slow
	return MakeShareable(new FDummyNameValidator(EValidatorResult::Ok));
}

FText UK2Node_SnapContainer::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::MenuTitle)
	{
		return LOCTEXT("AddSnapContainer", "Add Snap Container...");
	}
	
	return LOCTEXT("PlacedSnapContainerNodeTitle", "Snap Container");
}

void UK2Node_SnapContainer::ReconstructNode()
{
	if (!HasAnyFlags(RF_NeedLoad))
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		if (UK2Node_ExecutionSequence* Sequence = Cast<UK2Node_ExecutionSequence>(RootNode))
		{
			// Compact the sequence node, and make sure there is an empty entry at the end of the node (always keep an open drag target)
			//@TODO: Do that thing I said
			UEdGraphPin* LastExecPin = nullptr;
			for (UEdGraphPin* Pin : Sequence->Pins)
			{
				if (K2Schema->IsExecPin(*Pin) && (Pin->Direction == EGPD_Output))
				{
					LastExecPin = Pin;
				}
			}

			if ((LastExecPin == nullptr) || (LastExecPin->LinkedTo.Num() > 0))
			{
				Sequence->AddInputPin();
			}
		}
	}
	Super::ReconstructNode();
}

void UK2Node_SnapContainer::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

// 	// AllocateDefaultPins() gets called too early during initial node creation, before BoundGraph was created, so
// 	// the first time thru we need to add a pin manually.  After that it will be added automatically because the
// 	// inner tunnel has it
// 	if (FindPin(UEdGraphSchema_K2::PN_Execute, EGPD_Input) == nullptr)
// 	{
// 		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
// 		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, FString(), nullptr, UEdGraphSchema_K2::PN_Execute);
// 	}
}

void UK2Node_SnapContainer::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Create a root node (hardcoded to a sequence for now)
	UK2Node_ExecutionSequence* TopLevelSequence = SpawnNewNode<UK2Node_ExecutionSequence>(BoundGraph, FVector2D::ZeroVector);
	RootNode = TopLevelSequence;

	// Create an exec pin on the entry tunnel (gets mirrored onto the outer node automatically)
	UK2Node_Tunnel* EntryNode = GetEntryNode();
	FEdGraphPinType ExecPinType;
	ExecPinType.PinCategory = UEdGraphSchema_K2::PC_Exec;
	UEdGraphPin* TunnelExecIn = EntryNode->CreateUserDefinedPin(UEdGraphSchema_K2::PN_Execute, ExecPinType, EGPD_Input, /*bUseUniqueName=*/ false);

	// Make sure the inner tunnel is connected to the sequence node
	EntryNode->NodePosX = -200;
	UEdGraphPin* SequenceExec = TopLevelSequence->FindPinChecked(UEdGraphSchema_K2::PN_Execute);
	K2Schema->TryCreateConnection(TunnelExecIn, SequenceExec);

	// Position the exit node
	UK2Node_Tunnel* ExitNode = GetExitNode();
	ExitNode->NodePosX = 1024;

	// Make sure we have the top level exec pin to match the one we put into the entry node
	// (AllocateDefaultPins() gets called too early during initial node creation, before BoundGraph was created)
//	AllocateDefaultPins();
}

FText UK2Node_SnapContainer::GetKeywords() const
{
	return LOCTEXT("SnapContainerKeywords", "Block Node");
}

void UK2Node_SnapContainer::FindDiffs(class UEdGraphNode* OtherNode, struct FDiffResults& Results )
{
	UK2Node_SnapContainer* MathExpression1 = this;
	UK2Node_SnapContainer* MathExpression2 = Cast<UK2Node_SnapContainer>(OtherNode);

	// Compare the visual display of a math expression (the visual display involves consolidating variable Guid's into readable parameters)
	FText Expression1 = MathExpression1->GetNodeTitle(ENodeTitleType::EditableTitle);
	FText Expression2 = MathExpression2->GetNodeTitle(ENodeTitleType::EditableTitle);
	if (Expression1.CompareTo(Expression2) != 0)
	{
		FDiffSingleResult Diff;
		Diff.Node1 = MathExpression2;
		Diff.Node2 = MathExpression1;

		Diff.Diff = EDiffType::NODE_PROPERTY;
		FText NodeName = GetNodeTitle(ENodeTitleType::ListView);

		FFormatNamedArguments Args;
		Args.Add(TEXT("Expression1"), Expression1);
		Args.Add(TEXT("Expression2"), Expression2);

		Diff.ToolTip =  FText::Format(LOCTEXT("DIF_MathExpressionToolTip", "Math Expression '{Expression1}' changed to '{Expression2}'"), Args);
		Diff.Category = EDiffType::MODIFICATION;
		Diff.DisplayString = FText::Format(LOCTEXT("DIF_MathExpression", "Math Expression '{Expression1}' changed to '{Expression2}'"), Args);
		Results.Add(Diff);
	}
}

#undef LOCTEXT_NAMESPACE
