// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_Composite.h"
#include "K2Node_SnapContainer.generated.h"

class FBlueprintActionDatabaseRegistrar;
class INameValidatorInterface;

//@TODO: Write a comment
UCLASS()
class UK2Node_SnapContainer : public UK2Node_Composite
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TObjectPtr<UK2Node> RootNode;

public:
	UK2Node_SnapContainer(const FObjectInitializer& ObjectInitializer);

	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	virtual TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void ReconstructNode() override;
	virtual void FindDiffs(class UEdGraphNode* OtherNode, struct FDiffResults& Results) override;
	virtual bool ShouldMergeChildGraphs() const override { return true; }
	virtual void AllocateDefaultPins() override;
	virtual void PostPlacedNewNode() override;
	virtual FText GetKeywords() const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool IsNodePure() const { return false; }
	//~ End UK2Node Interface

	//~ Begin UK2Node_EditablePinBase Interface
	virtual bool CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage) override { return false; }
	//~ End UK2Node_EditablePinBase Interface

private:
	/** Cached so we don't have to regenerate it when the graph is recompiled */
	TSharedPtr<class FCompilerResultsLog> CachedMessageLog;
};


