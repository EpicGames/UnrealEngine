// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphNodeSnapContainer.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "GraphEditorSettings.h"
#include "SCommentBubble.h"
#include "SGraphPreviewer.h"
#include "IDocumentationPage.h"
#include "IDocumentation.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#include "SGraphSnapContainerRow.h"
#include "K2Node_SnapContainer.h"

#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "BlueprintSnapNodes"

/////////////////////////////////////////////////////
// SGraphNodeSnapContainer

void SGraphNodeSnapContainer::Construct(const FArguments& InArgs, UK2Node_SnapContainer* InNode)
{
	GraphNode = InNode;

	SetCursor(EMouseCursor::CardinalCross);

	UpdateGraphNode();
}

void SGraphNodeSnapContainer::UpdateGraphNode()
{
	InputPins.Empty();
	OutputPins.Empty();
	
	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	SetupErrorReporting();
	TSharedPtr<SNodeTitle> NodeTitle = SNew(SNodeTitle, GraphNode);


	UK2Node_SnapContainer* SnapContainer = CastChecked<UK2Node_SnapContainer>(GraphNode);
	TSharedRef<SWidget> SnapRegionWidget = FGraphSnapContainerBuilder::CreateSnapContainerWidgets(SnapContainer->BoundGraph, SnapContainer->RootNode);

	//
	//             ______________________
	//            |      TITLE AREA      |
	//            +-------+------+-------+
	//            | (>) L |      | R (>) |
	//            | (>) E |      | I (>) |
	//            | (>) F |      | G (>) |
	//            | (>) T |      | H (>) |
	//            |       |      | T (>) |
	//            |_______|______|_______|
	//
	this->GetOrAddSlot( ENodeZone::Center )
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush( "Graph.CollapsedNode.Body" ) )
			.Padding(0)
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				[
					SNew(SImage)
					.Image( FAppStyle::GetBrush("Graph.CollapsedNode.BodyColorSpill") )
					.ColorAndOpacity( this, &SGraphNode::GetNodeTitleColor )
				]
				+SOverlay::Slot()
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Top)
					[
						SNew(SOverlay)
						+SOverlay::Slot()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						[
							SNew(SBorder)
							.BorderImage( FAppStyle::GetBrush("NoBorder") )  // Graph.CollapsedNode.ColorSpill
							.Padding( FMargin(10,5,30,3) )
							[
								SNew(SVerticalBox)
								+SVerticalBox::Slot()
								.AutoHeight()
								.HAlign(HAlign_Fill)
								.VAlign(VAlign_Top)
								[
									SNew(SVerticalBox)
									+SVerticalBox::Slot()
										.AutoHeight()
									[
										SAssignNew(InlineEditableText, SInlineEditableTextBlock)
										.Style( FAppStyle::Get(), "Graph.Node.NodeTitleInlineEditableText" )
										.Text( NodeTitle.Get(), &SNodeTitle::GetHeadTitle )
										.OnVerifyTextChanged(this, &SGraphNodeSnapContainer::OnVerifyNameTextChanged)
										.OnTextCommitted(this, &SGraphNodeSnapContainer::OnNameTextCommited)
										.IsReadOnly( this, &SGraphNodeSnapContainer::IsNameReadOnly )
										.IsSelected(this, &SGraphNodeSnapContainer::IsSelectedExclusively)
									]
									+SVerticalBox::Slot()
										.AutoHeight()
									[
										NodeTitle.ToSharedRef()
									]
								]
								+SVerticalBox::Slot()
								.AutoHeight()
								.Padding(1.0f)
								[
									ErrorReporting->AsWidget()
								]
							]
						]
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Top)
					[
						CreateNodeBody()
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Top)
					[
						SnapRegionWidget
					]
				]
			]
		];
	// Create comment bubble
	TSharedPtr<SCommentBubble> CommentBubble;
	const FSlateColor CommentColor = GetDefault<UGraphEditorSettings>()->DefaultCommentNodeTitleColor;

	SAssignNew( CommentBubble, SCommentBubble )
	.GraphNode( GraphNode )
	.Text( this, &SGraphNode::GetNodeComment )
	.OnTextCommitted( this, &SGraphNode::OnCommentTextCommitted )
	.ColorAndOpacity( CommentColor )
	.AllowPinning( true )
	.EnableTitleBarBubble( true )
	.EnableBubbleCtrls( true )
	.GraphLOD( this, &SGraphNode::GetCurrentLOD )
	.IsGraphNodeHovered( this, &SGraphNode::IsHovered );

	GetOrAddSlot( ENodeZone::TopCenter )
	.SlotOffset2f( TAttribute<FVector2f>( CommentBubble.Get(), &SCommentBubble::GetOffset2f ))
	.SlotSize2f( TAttribute<FVector2f>( CommentBubble.Get(), &SCommentBubble::GetSize2f ))
	.AllowScaling( TAttribute<bool>( CommentBubble.Get(), &SCommentBubble::IsScalingAllowed ))
	.VAlign( VAlign_Top )
	[
		CommentBubble.ToSharedRef()
	];

	CreatePinWidgets();
}

UEdGraph* SGraphNodeSnapContainer::GetInnerGraph() const
{
	UK2Node_Composite* CompositeNode = CastChecked<UK2Node_Composite>(GraphNode);
	return CompositeNode->BoundGraph;
}

TSharedPtr<SToolTip> SGraphNodeSnapContainer::GetComplexTooltip()
{
	if (UEdGraph* BoundGraph = GetInnerGraph())
	{
		struct LocalUtils
		{
			static bool IsInteractive()
			{
				const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
				return ( ModifierKeys.IsAltDown() && ModifierKeys.IsControlDown() );
			}
		};

		TSharedPtr<SToolTip> FinalToolTip = NULL;
		TSharedPtr<SVerticalBox> Container = NULL;
		SAssignNew(FinalToolTip, SToolTip)
		.IsInteractive_Static(&LocalUtils::IsInteractive)
		[
			SAssignNew(Container, SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew( STextBlock )
				.Text(this, &SGraphNodeSnapContainer::GetTooltipTextForNode)
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
				.WrapTextAt(160.0f)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				// Create preview for the tooltip, make sure to disable state overlays to prevent
				// PIE and read-only borders obscuring the graph
				SNew(SGraphPreviewer, BoundGraph)
				.CornerOverlayText(this, &SGraphNodeSnapContainer::GetPreviewCornerText)
				.ShowGraphStateOverlay(false)
			]
		];

		// Check to see whether this node has a documentation excerpt. If it does, create a doc box for the tooltip
		TSharedRef<IDocumentationPage> DocPage = IDocumentation::Get()->GetPage(GraphNode->GetDocumentationLink(), NULL);
		if(DocPage->HasExcerpt(GraphNode->GetDocumentationExcerptName()))
		{
			Container->AddSlot()
			.AutoHeight()
			.Padding(FMargin( 0.0f, 5.0f ))
			[
				IDocumentation::Get()->CreateToolTip(FText::FromString("Documentation"), NULL, GraphNode->GetDocumentationLink(), GraphNode->GetDocumentationExcerptName())
			];
		}

		return FinalToolTip;
	}
	else
	{
		return SNew(SToolTip)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew( STextBlock )
					.Text(NSLOCTEXT("CompositeNode", "CompositeNodeInvalidGraphMessage", "ERROR: Invalid Graph"))
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
					.WrapTextAt(160.0f)
				]
			];
	}

}

FText SGraphNodeSnapContainer::GetPreviewCornerText() const
{
	return LOCTEXT("SnapGraphCornerText", "Snap Magic");
}

FText SGraphNodeSnapContainer::GetTooltipTextForNode() const
{
	return GraphNode->GetTooltipText();
}

TSharedRef<SWidget> SGraphNodeSnapContainer::CreateNodeBody()
{
	if( GraphNode && GraphNode->Pins.Num() > 0 )
	{
		// Create the input and output pin areas if there are pins
		return SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush("NoBorder") )
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding( FMargin(0,3) )
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.FillWidth(1.0f)
				[
					// LEFT
					SAssignNew(LeftNodeBox, SVerticalBox)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					// RIGHT
					SAssignNew(RightNodeBox, SVerticalBox)
				]
			];
	}
	else
	{
		// Create a spacer so the node has some body to it
		return SNew(SSpacer)
			.Size(FVector2D(100.f, 50.f));
	}
}

#undef LOCTEXT_NAMESPACE
