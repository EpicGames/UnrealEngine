// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KismetNodes/SGraphNodeK2Base.h"

class SToolTip;
class UEdGraph;
class UK2Node_SnapContainer;

class SGraphNodeSnapContainer : public SGraphNodeK2Base
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeSnapContainer){}
	SLATE_END_ARGS()
	typedef SGraphNodeK2Base Super;

	void Construct(const FArguments& InArgs, UK2Node_SnapContainer* InNode);

	// SGraphNode interface
	virtual void UpdateGraphNode() override;
	virtual TSharedPtr<SToolTip> GetComplexTooltip() override;
	// End of SGraphNode interface
protected:
	virtual UEdGraph* GetInnerGraph() const;

private:
	FText GetPreviewCornerText() const;
	FText GetTooltipTextForNode() const;

	TSharedRef<SWidget> CreateNodeBody();
};
