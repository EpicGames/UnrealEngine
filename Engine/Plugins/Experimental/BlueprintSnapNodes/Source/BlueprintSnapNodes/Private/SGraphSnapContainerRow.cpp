// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphSnapContainerRow.h"
#include "EdGraph/EdGraph.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"

#include "NodeFactory.h"
#include "K2Node_ExecutionSequence.h"
#include "SGraphPin.h"
#include "GraphEditorDragDropAction.h"
#include "K2Node_VariableGet.h"
#include "K2Node_VariableSet.h"
#include "EdGraphSchema_K2.h"
#include "Widgets/Layout/SBox.h"

#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "BlueprintSnapNodes"

//@TODO: Figure out how to handle self nodes in a reasonable way (it currently does "self.", could drag an Other onto it of the same type)

//@TODO: No provision for constructing one of these without drag drop (e.g., a dropdown to pick a method on a self pin)
//@TODO: Drag-dropping a variable onto an existing variable doesn't work right (not a bug here, same issue happens if you drag a float var onto a float input pin that already has a connection; it drops the get node but fails to replace the existing one)
//@TODO: How to deal with replaced stuff (after the above note is fixed), e.g., a node in the subgraph that is no longer wired to anything.  Could just clean unconnected nodes on each edit, but seems messy.

//@TODO: Tooltip is wrong on variable get nodes (and they're just ugly fake versions of the real thing)
//@TODO: Styling in general is ugly

//@TODO: Drag-dropping a connection onto one of the variable 'pins' should probably do something or provide an explicit tooltip why it won't
// (something might be adding an input pin to the composite that gets wired up internally and adding a generated name for the inner pin that gets wired up)
// This is a place where a lot of the stuff that math expression nodes do may be useful RE: any old variable 'works' and just becomes an input or output ref

//@TODO: This should probably get collapsed with the code in SPinTypeSelector::GetTypeDescription (this one is eventually going to be better!)
// It's not perfect tho (just does [] for array/set/map and & for ref
FText GetTypeDescription(const FEdGraphPinType& PinType)
{
	const FName PinSubCategory = PinType.PinSubCategory;
	const UObject* PinSubCategoryObject = PinType.PinSubCategoryObject.Get();

	FText BasicText;
	if ((PinSubCategory != UEdGraphSchema_K2::PSC_Bitmask) && (PinSubCategoryObject != nullptr))
	{
		if (const UField* Field = Cast<const UField>(PinSubCategoryObject))
		{
			BasicText = Field->GetDisplayNameText();
		}
		else
		{
			BasicText = FText::AsCultureInvariant(PinSubCategoryObject->GetName());
		}
	}
	else
	{
		BasicText = UEdGraphSchema_K2::GetCategoryText(PinType.PinCategory, true);
	}

	switch (PinType.ContainerType)
	{
	default:
		break;
	case EPinContainerType::Array:
		BasicText = FText::Format(LOCTEXT("PinTypeIsArray", "TArray<{0}>"), BasicText);
		break;
	case EPinContainerType::Set:
		BasicText = FText::Format(LOCTEXT("PinTypeIsSet", "TSet<{0}>"), BasicText);
		break;
	case EPinContainerType::Map:
		BasicText = FText::Format(LOCTEXT("PinTypeIsMap", "TMap<{0}, TODO>"), BasicText);
		break;
	}

	if (PinType.bIsConst)
	{
		BasicText = FText::Format(LOCTEXT("PinTypeIsConst", "const {0}"), BasicText);
	}

	if (PinType.bIsReference)
	{
		BasicText = FText::Format(LOCTEXT("PinTypeIsReference", "{0}&"), BasicText);
	}

	return BasicText;
}


/////////////////////////////////////////////////////
// FGraphSnapContainerBuilder

TSharedRef<SWidget> FGraphSnapContainerBuilder::CreateSnapContainerWidgets(UEdGraph* ModelGraph, UEdGraphNode* RootNode)
{
	FGraphSnapContainerBuilder Builder(ModelGraph);

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	UEdGraphPin* RootPin = Schema->FindExecutionPin(*RootNode, EGPD_Input);

	return Builder.MakeNodeWidget(RootNode, RootPin);
}

FGraphSnapContainerBuilder::FGraphSnapContainerBuilder(UEdGraph* InGraph)
	: Graph(InGraph)
{
}

TSharedRef<SWidget> FGraphSnapContainerBuilder::MakeNodeWidget(UEdGraphNode* Node, UEdGraphPin* FromPin)
{
	// Make sure we visit each node only once; to prevent infinite recursion
	bool bAlreadyInSet = false;
	VisitedNodes.Add(Node, &bAlreadyInSet);
	if (bAlreadyInSet)
	{
		return SNullWidget::NullWidget;
		// SNew(STextBlock).Text(LOCTEXT("RecursionOccurredInNodeGraphMessage", "RECURSION"));
	}

	// Evaluate the node to gather information that many of the widgets will want
	const UEdGraphSchema* Schema = Node->GetSchema();
	TArray<UEdGraphPin*> InputPins;
	TArray<UEdGraphPin*> OutputPins;
	int32 InputPinCount = 0;
	int32 OutputPinCount = 0;
	for (auto PinIt = Node->Pins.CreateConstIterator(); PinIt; ++PinIt)
	{
		UEdGraphPin* Pin = *PinIt;
		if (!Pin->bHidden)
		{
			if (Pin->Direction == EGPD_Input)
			{
				InputPins.Add(Pin);
				++InputPinCount;
			}
			else
			{
				OutputPins.Add(Pin);
				++OutputPinCount;
			}
		}
	}

	// Determine if the node is impure
	UK2Node* K2Node = Cast<UK2Node>(Node);
	const bool bIsPure = (K2Node != nullptr) && K2Node->IsNodePure();

	if (UK2Node_ExecutionSequence* Sequence = Cast<UK2Node_ExecutionSequence>(Node))
	{
		TSharedRef<SVerticalBox> VerticalPinBox = SNew(SVerticalBox);

		for (UEdGraphPin* OutputExecPin : OutputPins)
		{
			VerticalPinBox->AddSlot()
			.Padding(FMargin(12.0f, 0.0f, 12.0f, 4.0f))
			.AutoHeight()
			[
				SNew(SBox)
				.Padding(FMargin(8.0f, 3.0f))
				[
					MakePinWidget(OutputExecPin)
				]
			];
		}

		return VerticalPinBox;
	}
	else if ((OutputPinCount != 1) || !bIsPure)
	{
		// It's a function call
		check(K2Node);
		return MakeFunctionCallWidget(K2Node);

#if 0
		// The source node is impure or has multiple outputs, so cannot be directly part of this pure expression
		// Instead show it as a special sort of variable get
		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeTitle"), Node->GetNodeTitle(ENodeTitleType::ListView));
		Args.Add(TEXT("PinName"), FromPin->GetDisplayName());
		const FText EffectiveVariableName = FText::Format(LOCTEXT("NodeTitleWithPinNameImpure", "{NodeTitle}_{PinName} impure"), Args );

		return SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image( FAppStyle::GetBrush("KismetExpression.ReadAutogeneratedVariable.Body") )
				.ColorAndOpacity( Schema->GetPinTypeColor(FromPin->PinType) )
			]
			+ SOverlay::Slot()
			.Padding( FMargin(6,4) )
			[
				SNew(STextBlock)
				.TextStyle( FAppStyle::Get(), TEXT("KismetExpression.ReadAutogeneratedVariable") )
				.Text( EffectiveVariableName )
			];
#endif
	}
	else if (auto VarGetNode = Cast<const UK2Node_VariableGet>(Node))
	{
		// Variable get node
		return SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image( FAppStyle::GetBrush("KismetExpression.ReadVariable.Body") )
				.ColorAndOpacity( Schema->GetPinTypeColor(OutputPins[0]->PinType) )
			]
			+ SOverlay::Slot()
			.Padding( FMargin(6,4) )
			[
				SNew(STextBlock)
				.TextStyle( FAppStyle::Get(), TEXT("KismetExpression.ReadVariable") )
				.Text( FText::FromString( VarGetNode->GetVarNameString() ) )
			]
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Visibility(EVisibility::HitTestInvisible)
				.Image( FAppStyle::GetBrush("KismetExpression.ReadVariable.Gloss") )
			];
		/*
			+ SOverlay::Slot()
			.Padding( FMargin(6,4) )
			[
				SNew(STextBlock)
				.TextStyle( FAppStyle::Get(), TEXT("KismetExpression.ReadVariable") )
				.Text( VarGetNode->VariableName.ToString() )
			];
			*/
	}
	else if (UK2Node* AnyNode = Cast<UK2Node>(Node))
	{
		const bool bIsCompact = AnyNode->ShouldDrawCompact() && (InputPinCount <= 2);

		TSharedRef<SWidget> OperationWidget = 
			SNew(STextBlock)
			.TextStyle( FAppStyle::Get(), bIsCompact ? TEXT("KismetExpression.OperatorNode") : TEXT("KismetExpression.FunctionNode") )
			.Text(AnyNode->GetCompactNodeTitle());

		if ((InputPinCount == 1) && bIsCompact)
		{
			// One-pin compact nodes are assumed to be unary operators
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4, 0)
				[
					OperationWidget
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					MakePinWidget(InputPins[0])
				];
		}
		else if ((InputPinCount == 2) && bIsCompact)
		{
			// Two-pin compact nodes are assumed to be binary operators
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					MakePinWidget(InputPins[0])
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4, 0)
				[
					OperationWidget
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					MakePinWidget(InputPins[1])
				];
		}
		else
		{
			// All other operations are treated as traditional function calls
			return MakeFunctionCallWidget(AnyNode);
		}
	}
	else
	{
		return SNew(STextBlock).Text(LOCTEXT("UnknownNodeMessage", "UNKNOWN_NODE"));
	}
}

TSharedRef<SWidget> FGraphSnapContainerBuilder::MakeFunctionCallWidget(UK2Node* AnyNode)
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	TSharedRef<SHorizontalBox> WidgetContainer = SNew(SHorizontalBox);

	// Add the return value (if any)
	UEdGraphPin* ReturnValuePin = AnyNode->FindPin(UEdGraphSchema_K2::PN_ReturnValue);
	UEdGraphPin* SelfPin = Schema->FindSelfPin(*AnyNode, EGPD_Input);

	if (ReturnValuePin != nullptr)
	{
		WidgetContainer->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4)
			[
				MakePinWidget(ReturnValuePin)
			];

		WidgetContainer->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AssignmentExpression", "="))
			];
	}

	// Add the scope if needed
	if ((SelfPin != nullptr) && !SelfPin->bHidden)
	{
		WidgetContainer->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				MakePinWidget(SelfPin)
			];

		WidgetContainer->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(2.0f, 0.0f))
			[
				SNew(STextBlock).Text(LOCTEXT("ScopingOperator", "."))
			];
	}

	// Add the function name
	TSharedRef<SWidget> OperationWidget =
		SNew(STextBlock)
		.TextStyle(FAppStyle::Get(), TEXT("KismetExpression.FunctionNode"))
		.Text(AnyNode->GetCompactNodeTitle());
	WidgetContainer->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			OperationWidget
		];

	// Create the argument list
	TSharedRef<SHorizontalBox> InnerBox =
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BeginExpression", "("))
		];

	bool bFirstPin = true;
	for (UEdGraphPin* Pin : AnyNode->Pins)
	{
		if (Schema->IsExecPin(*Pin) || (Pin == ReturnValuePin) || (Pin->bHidden) || (Pin == SelfPin))
		{
			continue;
		}

		//@TODO: This is not general enough, some pins are 'stupid'
		// Hide the output variable pin if it's a variable set node and it has no connections, ugh!
		if (Cast<UK2Node_VariableSet>(AnyNode) != nullptr)
		{
			if ((Pin->Direction == EGPD_Output) && (Pin->LinkedTo.Num() == 0))
			{
				continue;
			}
		}

		// Add the comma if needed
		if (!bFirstPin)
		{
			InnerBox->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.0f,0.0f))
			[
				SNew(STextBlock).Text(LOCTEXT("NextExpression", ","))
			];
		}
		bFirstPin = false;

		// Add an indicator for output pins
		if (Pin->Direction == EGPD_Output)
		{
			InnerBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("OutPin", "out"))
				];
		}
		else if (Pin->PinType.bIsReference)
		{
			InnerBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("RefPin", "ref "))
				];
		}

		// Add the pin contents
		InnerBox->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			MakePinWidget(Pin)
		];
	}

	InnerBox->AddSlot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	.Padding(4)
	[
		SNew(STextBlock).Text(LOCTEXT("EndExpression", ")"))
	];
	WidgetContainer->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			InnerBox
		];

	return WidgetContainer;
}

TSharedRef<SWidget> FGraphSnapContainerBuilder::MakePinWidget(UEdGraphPin* Pin)
{
	check(Pin);
	
	TSharedPtr<SWidget> ChildWidget = nullptr;
	if (Pin->LinkedTo.Num() > 0)
	{
		UEdGraphPin* ChildPin = Pin->LinkedTo[0];
		ChildWidget = MakeNodeWidget(ChildPin->GetOwningNode(), ChildPin->LinkedTo[0]);
	}

	return SNew(SGraphSnapContainerEntry, Pin->GetOwningNode(), Pin, ChildWidget);
}

/////////////////////////////////////////////////////
// SGraphSnapContainerEntry

void SGraphSnapContainerEntry::Construct(const FArguments& InArgs, UEdGraphNode* InNode, UEdGraphPin* InPin, TSharedPtr<SWidget> InChildWidget)
{
	TargetNode = InNode;
	check(TargetNode);
	TargetPin = InPin;
	check(TargetPin);

	FLinearColor BackgroundColor = FLinearColor::Red;
	TSharedRef<SWidget> ContentToShow = InChildWidget.IsValid() ? InChildWidget.ToSharedRef() : SNullWidget::NullWidget;

	BackgroundColor = TargetPin->GetSchema()->GetPinTypeColor(TargetPin->PinType);

	if (!InChildWidget.IsValid())
	{
		// This should only be called on pins that have nothing hooked up to them (without passing in an inner widget!)
		check(TargetPin->LinkedTo.Num() == 0);

		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
		if (Schema->IsExecPin(*TargetPin))
		{
			ContentToShow = SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), TEXT("KismetExpression.LiteralValue"))
				.Text(LOCTEXT("SnapContainerExecutionPinDesc", "Drag nodes here"));
				//.ColorAndOpacity(FLinearColor::Black);
		}
		else
		{
			// Try showing the literal field for the pin
			if (TargetPin->Direction == EGPD_Input)
			{
				DefaultValuePinWidget = FNodeFactory::CreatePinWidget(TargetPin);
				DefaultValuePinWidget->SetOnlyShowDefaultValue(true);

				ContentToShow = DefaultValuePinWidget->GetDefaultValueWidget();
				if (TargetPin->PinType.bIsReference)
				{
					DefaultValuePinWidget.Reset();
					ContentToShow = SNullWidget::NullWidget;
				}
			}

			// Failing that, show the type description
			if (ContentToShow == SNullWidget::NullWidget)
			{
				ContentToShow = SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), TEXT("KismetExpression.LiteralValue"))
					.Text(GetTypeDescription(TargetPin->PinType));
			}
		}
	}

	SetVisibility(EVisibility::Visible);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("PlainBorder"))
		.BorderBackgroundColor(FSlateColor(BackgroundColor))
		[
			SNew(SBox)
			.MinDesiredWidth(40.0f)
			.MinDesiredHeight(20.0f)
			.Padding(FMargin(0.0f, 0.0f))
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				ContentToShow
			]
		]
	];
}


FReply SGraphSnapContainerEntry::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	//@TODO:
//	bool bReadOnly = OwnerGraphPanelPtr.IsValid() ? !OwnerGraphPanelPtr.Pin()->IsGraphEditable() : false;
	const bool bReadOnly = false;

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid() || bReadOnly)
	{
		return FReply::Unhandled();
	}

	// Is someone dropping a connection onto this node?
	if (Operation->IsOfType<FGraphEditorDragDropAction>())
	{
		TSharedPtr<FGraphEditorDragDropAction> DragConnectionOp = StaticCastSharedPtr<FGraphEditorDragDropAction>(Operation);

		check(TargetNode);
		check(TargetPin);
		DragConnectionOp->SetHoveredGraph(nullptr);
		DragConnectionOp->SetHoveredNode(TargetNode);
		DragConnectionOp->SetHoveredPin(TargetPin);

		//const FVector2f NodeAddPosition = NodeCoordToGraphCoord(MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition()));
		const FVector2f NodeAddPosition = FVector2f(TargetNode->NodePosX, TargetNode->NodePosY) +
			FVector2f(0.0f, 64.0f) * TargetNode->Pins.Find(TargetPin) +
			FVector2f((TargetPin->Direction == EGPD_Output) ? 200.0f : -180.0f, 0.0f);

		FReply Result = DragConnectionOp->DroppedOnPin(DragDropEvent.GetScreenSpacePosition(), NodeAddPosition);

		if (Result.IsEventHandled())
		{
			TargetNode->GetGraph()->NotifyGraphChanged();

			//@TODO: Need to tell ourselves we've changed too!
			// This is doing so, but it's pretty hacky
			if (UK2Node* CompositeNode = Cast<UK2Node>(TargetNode->GetGraph()->GetOuter()))
			{
				CompositeNode->ReconstructNode();
				CompositeNode->GetGraph()->NotifyGraphChanged();
			}
		}
		return Result;
	}

	return FReply::Unhandled();
}

TSharedPtr<IToolTip> SGraphSnapContainerEntry::GetToolTip()
{
	// We just masquerade as a pin, so grab the underlying pin tooltip here
	if (DefaultValuePinWidget.IsValid())
	{
		return DefaultValuePinWidget->GetToolTip();
	}

	return SWidget::GetToolTip();
}

FReply SGraphSnapContainerEntry::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
// 		FDragConnection::FDraggedPinTable PinHandles;
// 		PinHandles.Reserve(InStartingPins.Num());
// 		// since the graph can be refreshed and pins can be reconstructed/replaced 
// 		// behind the scenes, the DragDropOperation holds onto FGraphPinHandles 
// 		// instead of direct widgets/graph-pins
// 		for (const TSharedRef<SGraphPin>& PinWidget : InStartingPins)
// 		{
// 			PinHandles.Add(PinWidget->GetPinObj());
// 		}
// 
// 		TSharedRef<FDragConnection> DragDropOp = FDragConnection::New(InGraphPanel, PinHandles);
// 
// 		return FReply::Handled().BeginDragDrop(DragDropOp);
	}

	return FReply::Unhandled();
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
