// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;

class SGraphPin;
class UK2Node;

/////////////////////////////////////////////////////
// FGraphSnapContainerBuilder

class FGraphSnapContainerBuilder
{
public:
	static TSharedRef<SWidget> CreateSnapContainerWidgets(UEdGraph* ModelGraph, UEdGraphNode* RootNode);

private:
	FGraphSnapContainerBuilder(UEdGraph* InGraph);

	TSharedRef<SWidget> MakeNodeWidget(UEdGraphNode* Node, UEdGraphPin* FromPin);
	TSharedRef<SWidget> MakePinWidget(UEdGraphPin* Pin);
	TSharedRef<SWidget> MakeFunctionCallWidget(UK2Node* AnyNode);
private:
	UEdGraph* Graph;
	TSet<const UEdGraphNode*> VisitedNodes;
};

/////////////////////////////////////////////////////
// SGraphSnapContainerEntry

class SGraphSnapContainerEntry : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGraphSnapContainerEntry){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphNode* InNode, UEdGraphPin* InPin, TSharedPtr<SWidget> InChildWidget);

	// SWidget interface
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual TSharedPtr<IToolTip> GetToolTip() override;
	// End of SWidget interface

private:
	UEdGraphNode* TargetNode;
	UEdGraphPin* TargetPin;

	TSharedPtr<SGraphPin> DefaultValuePinWidget;
};
